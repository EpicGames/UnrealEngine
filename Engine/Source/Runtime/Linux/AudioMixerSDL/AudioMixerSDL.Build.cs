// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioMixerSDL : ModuleRules
{
	public AudioMixerSDL(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.Add("TargetPlatform");
		PrivateIncludePathModuleNames.Add("AudioMixer");

		string PlatformName = Target.Platform.ToString();
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PlatformName = "Linux";
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateIncludePaths.Add("Runtime/Linux/AudioMixerSDL/Private/" + PlatformName);
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"AudioMixer",
				"AudioMixerCore"
			}
			);

		AddEngineThirdPartyPrivateStaticDependencies(Target, 
			"UEOgg",
			"Vorbis",
			"VorbisFile",
			"SDL3"
			);
	}
}
