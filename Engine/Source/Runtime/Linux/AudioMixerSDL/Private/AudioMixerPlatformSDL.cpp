// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMixerPlatformSDL.h"
#include "Modules/ModuleManager.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "AudioDevice.h"

#if WITH_ENGINE
#include "AudioPluginUtilities.h"
#endif // WITH_ENGINE

DECLARE_LOG_CATEGORY_EXTERN(LogAudioMixerSDL, Log, All);
DEFINE_LOG_CATEGORY(LogAudioMixerSDL);

namespace Audio
{
	// Static callback function used in SDL

	static void SDLCALL OnBufferEndWrapper(void* UserData, SDL_AudioStream* Stream, int AdditionalAmount, int TotalAmount)
	{
		if (AdditionalAmount > 0)
		{
			FMixerPlatformSDL* MixerPlatform = (FMixerPlatformSDL*)UserData;
			int OutputBufferByteLength = MixerPlatform->GetOutputBufferByteLength();

			Uint8* Data = SDL_stack_alloc(Uint8, OutputBufferByteLength);
			if (Data)
			{
				MixerPlatform->HandleOnBufferEnd(Data, OutputBufferByteLength);
				SDL_PutAudioStreamData(Stream, Data, OutputBufferByteLength);
				SDL_stack_free(data);
			}
		}
	}

	FMixerPlatformSDL::FMixerPlatformSDL()
		: AudioDeviceID(INDEX_NONE)
		, AudioStream(nullptr)		
		, OutputBuffer(nullptr)
		, OutputBufferByteLength(0)
		, bSuspended(false)
		, bInitialized(false)
	{
	}

	FMixerPlatformSDL::~FMixerPlatformSDL()
	{
		if (bInitialized)
		{
			TeardownHardware();
		}
	}

	bool FMixerPlatformSDL::InitializeHardware()
	{
		if (bInitialized)
		{
			UE_LOGF(LogAudioMixerSDL, Error, "SDL Audio already initialized.");
			return false;
		}

		// If we're rendering offscreen, use the "dummy" SDL audio driver
		if (FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")) && !getenv("SDL_AUDIODRIVER"))
		{
			UE_LOGF(LogAudioMixerSDL, Log, "Hinting SDL to use 'dummy' audio driver.");
			SDL_SetHint(SDL_HINT_AUDIO_DRIVER, "dummy");
		}


		int32 Result = SDL_InitSubSystem(SDL_INIT_AUDIO);
		if (Result < 0)
		{
			UE_LOGF(LogAudioMixerSDL, Error, "SDL_InitSubSystem create failed: %d", Result);
			return false;
		}

		const char* DriverName = SDL_GetCurrentAudioDriver();
		UE_LOGF(LogAudioMixerSDL, Display, "Initialized SDL using %ls platform API backend.", ANSI_TO_TCHAR(DriverName));

		if (IAudioMixer::ShouldRecycleThreads())
		{
			// Pre-create the null render device thread so we can simple wake it up when we need it.
			// Give it nothing to do, with a slow tick as the default, but ask it to wait for a signal to wake up.
			CreateNullDeviceThread([] {}, 1.0f, true);
		}

		bInitialized = true;
		return true;
	}

	bool FMixerPlatformSDL::TeardownHardware()
	{
		if(!bInitialized)
		{
			return true;
		}

		StopAudioStream();
		CloseAudioStream();

		// this is refcounted
		SDL_QuitSubSystem(SDL_INIT_AUDIO);
		bInitialized = false;

		return true;
	}

	bool FMixerPlatformSDL::IsInitialized() const
	{
		return bInitialized;
	}

	bool FMixerPlatformSDL::GetNumOutputDevices(uint32& OutNumOutputDevices)
	{
		if (!bInitialized)
		{
			UE_LOGF(LogAudioMixerSDL, Error, "SDL3 Audio is not initialized.");
			return false;
		}

		int NumDevices = 0;
		SDL_AudioDeviceID* Devices = SDL_GetAudioPlaybackDevices(&NumDevices);
		OutNumOutputDevices = NumDevices;
		SDL_free(Devices);
		return true;
	}

	bool FMixerPlatformSDL::GetOutputDeviceInfo(const uint32 InDeviceID, FAudioPlatformDeviceInfo& OutInfo)
	{
		FAudioPlatformSettings PlatformSettings = GetPlatformSettings();
		AudioSpec  = { GetPlatformAudioFormat(), GetPlatformChannels(), PlatformSettings.SampleRate };

		FString DeviceName;
		int NumDevices = 0;
		SDL_AudioDeviceID* Devices = SDL_GetAudioPlaybackDevices(&NumDevices);

		if (NumDevices)
		{
			SDL_AudioDeviceID DevId;

			if (InDeviceID != AUDIO_MIXER_DEFAULT_DEVICE_INDEX)
			{
				DevId = Devices[InDeviceID];
			}
			else
			{
				DevId = SDL_AUDIO_DEVICE_DEFAULT_PLAYBACK;
			}

			const char* AudioDeviceName = nullptr;

			AudioDeviceName = SDL_GetAudioDeviceName(DevId);
			DeviceName = ANSI_TO_TCHAR(AudioDeviceName);

			SDL_free(Devices);
			Devices = nullptr;
		}


		// opening a physical device no longer changes the spec we pass in, so there's no reason to bother with any of the previous code.
		// Just take the device instance we have and return exactly what we asked for.  The backend automatically takes care of making sure
		// the physical device is actually set to a capable sample rate, etc to accommodate our requested format
		OutInfo.DeviceId = DeviceName;
		OutInfo.Name = DeviceName;
		OutInfo.SampleRate = PlatformSettings.SampleRate;	// sample rate conversion now happens in the SDL audio code, so our code can send whatever we like
		OutInfo.Format = GetAudioStreamFormat();
		OutInfo.NumChannels = FMath::Min<int32>(static_cast<int32>(GetPlatformChannels()), AUDIO_MIXER_MAX_OUTPUT_CHANNELS);
		
		// Assume default channel map order, SDL doesn't support us querying it directly
		OutInfo.OutputChannelArray.Reset();
		for (int32 i = 0; i < OutInfo.NumChannels; ++i)
		{
			OutInfo.OutputChannelArray.Add(EAudioMixerChannel::Type(i));
		}

		return true;
	}

	bool FMixerPlatformSDL::GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const
	{
		// It's not possible to know what index the default audio device is.
		OutDefaultDeviceIndex = AUDIO_MIXER_DEFAULT_DEVICE_INDEX;
		return true;
	}

	bool FMixerPlatformSDL::OpenAudioStream(const FAudioMixerOpenStreamParams& Params)
	{
		if (!bInitialized || AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			return false;
		}

		int NumOutputDevices = 0;
		SDL_AudioDeviceID* Devices = SDL_GetAudioPlaybackDevices(&NumOutputDevices);
		if (!Devices || NumOutputDevices == 0)
		{
			return false;
		}

		OpenStreamParams = Params;

		if (OpenStreamParams.OutputDeviceIndex!= AUDIO_MIXER_DEFAULT_DEVICE_INDEX)
		{
			AudioDeviceID =  Devices[OpenStreamParams.OutputDeviceIndex];
		}
		else
		{
			AudioDeviceID =  SDL_AUDIO_DEVICE_DEFAULT_PLAYBACK;
		}
		SDL_free(Devices);
		Devices = nullptr;

		AudioStreamInfo.Reset();
		AudioStreamInfo.AudioMixer = OpenStreamParams.AudioMixer;
		AudioStreamInfo.NumOutputFrames = OpenStreamParams.NumFrames;
		AudioStreamInfo.NumBuffers = OpenStreamParams.NumBuffers;
		AudioStreamInfo.DeviceInfo.SampleRate = OpenStreamParams.SampleRate;

		// If there's an available device, open it.
		if (NumOutputDevices > 0)
		{		
			if (!GetOutputDeviceInfo(AudioStreamInfo.OutputDeviceIndex, AudioStreamInfo.DeviceInfo))
			{
				return false;
			}

			AudioStreamInfo.OutputDeviceIndex = OpenStreamParams.OutputDeviceIndex;

			AudioSpec.format = GetPlatformAudioFormat();
			AudioSpec.freq = Params.SampleRate;
			AudioSpec.channels = AudioStreamInfo.DeviceInfo.NumChannels;

			const char* DeviceName = nullptr;
			DeviceName = SDL_GetAudioDeviceName(AudioDeviceID);

			// only the default device can be overriden
			FString CurrentDeviceNameString = GetCurrentDeviceName();
			if (OpenStreamParams.OutputDeviceIndex != AUDIO_MIXER_DEFAULT_DEVICE_INDEX || CurrentDeviceNameString.Len() <= 0)
			{
				UE_LOGF(LogAudioMixerSDL, Log, "Opening %ls audio device (device index %d)", DeviceName ? ANSI_TO_TCHAR(DeviceName) : TEXT("default"), OpenStreamParams.OutputDeviceIndex);
			}
			else
			{
				UE_LOGF(LogAudioMixerSDL, Log, "Opening overridden '%ls' audio device (device index %d)", *CurrentDeviceNameString, OpenStreamParams.OutputDeviceIndex);
			}
	
			AudioStream = SDL_OpenAudioDeviceStream(AudioDeviceID, &AudioSpec, OnBufferEndWrapper, (void*)this);

			if (!AudioStream)
			{
				const char* ErrorText = SDL_GetError();
				UE_LOGF(LogAudioMixerSDL, Error, "%ls", ANSI_TO_TCHAR(ErrorText));
				return false;
			}

			// Compute the expected output byte length
			OutputBufferByteLength = OpenStreamParams.NumFrames * AudioStreamInfo.DeviceInfo.NumChannels * GetAudioStreamChannelSize();
		}
		else
		{
			// No devices available, Switch to NULL (Silent) Output.
			AudioStreamInfo.DeviceInfo.OutputChannelArray = { EAudioMixerChannel::FrontLeft, EAudioMixerChannel::FrontRight };
			AudioStreamInfo.DeviceInfo.NumChannels = 2;			
			AudioStreamInfo.DeviceInfo.Format = EAudioMixerStreamDataFormat::Float;
		}

		bInitialized = true;
		AudioStreamInfo.StreamState = EAudioOutputStreamState::Open;

		return true;
	}

	bool FMixerPlatformSDL::CloseAudioStream()
	{
		if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Closed)
		{
			return false;
		}

		if (!StopAudioStream())
		{
			return false;
		}

		if (AudioStream != nullptr)
		{
			FScopeLock ScopedLock(&OutputBufferMutex);

			SDL_DestroyAudioStream(AudioStream);
			AudioStream = nullptr;
			OutputBuffer = nullptr;
			OutputBufferByteLength = 0;
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Closed;
		return true;
	}

	bool FMixerPlatformSDL::StartAudioStream()
	{
		if (!bInitialized || (AudioStreamInfo.StreamState != EAudioOutputStreamState::Open && AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped))
		{
			return false;
		}

		// Start generating audio
		BeginGeneratingAudio();

		if (AudioStream != nullptr)
		{
			// Unpause audio device to start it rendering audio
			SDL_ResumeAudioStreamDevice(AudioStream);
		}
		else
		{
			check(!bIsUsingNullDevice);
			StartRunningNullDevice();
		}

		AudioStreamInfo.StreamState = EAudioOutputStreamState::Running;

		return true;
	}

	bool FMixerPlatformSDL::StopAudioStream()
	{
		if (!bInitialized)
		{
			return false;
		}

		if (AudioStreamInfo.StreamState != EAudioOutputStreamState::Stopped && AudioStreamInfo.StreamState != EAudioOutputStreamState::Closed)
		{
			if (AudioStream != nullptr)
			{
				// Pause the audio device
				SDL_PauseAudioStreamDevice(AudioStream);
			}
			else
			{
				check(bIsUsingNullDevice);
				StopRunningNullDevice();
			}

			if (AudioStreamInfo.StreamState == EAudioOutputStreamState::Running)
			{
				StopGeneratingAudio();
			}

			check(AudioStreamInfo.StreamState == EAudioOutputStreamState::Stopped);
		}

		return true;
	}

	FAudioPlatformDeviceInfo FMixerPlatformSDL::GetPlatformDeviceInfo() const
	{
		return AudioStreamInfo.DeviceInfo;
	}

	void FMixerPlatformSDL::SubmitBuffer(const uint8* Buffer)
	{
		// Need to prevent the case in which we close down the audio stream leaving this point to potentially corrupt the free'ed pointer
		FScopeLock ScopedLock(&OutputBufferMutex);

		if (OutputBuffer)
		{
			FMemory::Memcpy(OutputBuffer, Buffer, OutputBufferByteLength);
		}
	}

	void FMixerPlatformSDL::HandleOnBufferEnd(uint8* InOutputBuffer, int32 InOutputBufferByteLength)
	{
		if (!bIsDeviceInitialized)
		{
			return;
		}

		OutputBuffer = InOutputBuffer;
		check(InOutputBufferByteLength == OutputBufferByteLength);

		ReadNextBuffer();
	}

	FString FMixerPlatformSDL::GetDefaultDeviceName()
	{
		static FString DefaultName("Default SDL Audio Device.");
		return DefaultName;
	}

	void FMixerPlatformSDL::ResumeContext()
	{
		if (bSuspended)
		{
			if (AudioStream != nullptr)
			{
				SDL_ResumeAudioStreamDevice(AudioStream);
				UE_LOGF(LogAudioMixerSDL, Display, "Resuming Audio");
				bSuspended = false;
	
			}
		}
	}

	void FMixerPlatformSDL::SuspendContext()
	{
		if (!bSuspended)
		{
			if (AudioStream != nullptr)
			{
				SDL_PauseAudioStreamDevice(AudioStream);
				UE_LOGF(LogAudioMixerSDL, Display, "Suspending Audio");
				bSuspended = true;
			}
		}
	}

	FAudioPlatformSettings FMixerPlatformSDL::GetPlatformSettings() const
	{
#if PLATFORM_UNIX
		return FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
#else
		// On Windows, use default parameters.
		return FAudioPlatformSettings();
#endif
	}
}
