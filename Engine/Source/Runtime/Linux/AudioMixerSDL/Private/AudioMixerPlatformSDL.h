// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"

THIRD_PARTY_INCLUDES_START
#include <SDL3/SDL.h>
#include <SDL3/SDL_audio.h>
THIRD_PARTY_INCLUDES_END

namespace Audio
{

	class FMixerPlatformSDL : public IAudioMixerPlatformInterface
	{

	public:

		FMixerPlatformSDL();
		virtual ~FMixerPlatformSDL();

		//~ Begin IAudioMixerPlatformInterface Interface
		FString GetPlatformApi() const override { return TEXT("SDL3"); }
		bool InitializeHardware() override;
		bool TeardownHardware() override;
		bool IsInitialized() const override;
		bool GetNumOutputDevices(uint32& OutNumOutputDevices) override;
		bool GetOutputDeviceInfo(const uint32 InDeviceIndex, FAudioPlatformDeviceInfo& OutInfo) override;
		bool GetDefaultOutputDeviceIndex(uint32& OutDefaultDeviceIndex) const override;
		virtual bool OpenAudioStream(const FAudioMixerOpenStreamParams& Params) override;
		bool CloseAudioStream() override;
		bool StartAudioStream() override;
		bool StopAudioStream() override;
		FAudioPlatformDeviceInfo GetPlatformDeviceInfo() const override;
		void SubmitBuffer(const uint8* Buffer) override;
		FString GetDefaultDeviceName() override;
		FAudioPlatformSettings GetPlatformSettings() const override;
		void ResumeContext() override;
		void SuspendContext() override;
		//~ End IAudioMixerPlatformInterface Interface

		void HandleOnBufferEnd(uint8* InOutputBuffer, int32 InOutputBufferLength);

		SDL_AudioFormat GetPlatformAudioFormat() { return SDL_AUDIO_F32LE; }
		Uint8 GetPlatformChannels() { return 6; }
		EAudioMixerStreamDataFormat::Type GetAudioStreamFormat() { return EAudioMixerStreamDataFormat::Float; }
		int32 GetAudioStreamChannelSize() { return sizeof(float); }
		int32 GetOutputBufferByteLength() { return OutputBufferByteLength; }
		
	protected:
		FCriticalSection OutputBufferMutex;

	private:

		SDL_AudioDeviceID AudioDeviceID;
		SDL_AudioSpec AudioSpec;
		SDL_AudioStream *AudioStream;

		uint8* OutputBuffer;
		int32 OutputBufferByteLength;

		bool bSuspended;
		bool bInitialized;
	};

}

