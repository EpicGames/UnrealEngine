// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "AudioMixerPlatformSDL.h"
#include COMPILED_PLATFORM_HEADER(AudioMixerSDLDefines.h)

class FAudioMixerModuleSDL : public IAudioDeviceModule
{
public:
	virtual bool IsAudioMixerModule() const override { return true; }

	virtual void StartupModule() override
	{
		IAudioDeviceModule::StartupModule();

		FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
	}

	virtual FAudioDevice* CreateAudioDevice() override
	{
		return new Audio::FMixerDevice(new Audio::FAudioMixerPlatformSDL());
	}

	virtual Audio::IAudioMixerPlatformInterface* CreateAudioMixerPlatformInterface() override
	{
		return new Audio::FAudioMixerPlatformSDL();
	}
};

IMPLEMENT_MODULE(FAudioMixerModuleSDL, AudioMixerSDL);
