// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerPlatformSDL.h"

namespace Audio
{
	using FAudioMixerPlatformSDL = FMixerPlatformSDL;
}
